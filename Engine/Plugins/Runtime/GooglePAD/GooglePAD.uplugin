{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "GooglePAD",
	"Description" : "Google Play Asset Delivery for Android.",
	"Category" : "Android",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "GooglePAD",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" : [ "Android", "Win64", "Mac", "Linux" ]
		},
		{
			"Name" : "GooglePADEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Win64", "Mac", "Linux" ]
		}
	]
}
