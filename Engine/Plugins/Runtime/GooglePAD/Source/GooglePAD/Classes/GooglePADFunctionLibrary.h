// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GooglePADFunctionLibrary.generated.h"

#define SUPPORTED_PLATFORM (PLATFORM_ANDROID && USE_ANDROID_JNI)

#if SUPPORTED_PLATFORM
#include "play/asset_pack.h"
#endif

// An error code associated with Asset Pack operations.
UENUM(BlueprintType)
enum class EGooglePADErrorCode : uint8
{
	// There was no error with the request.
	AssetPack_NO_ERROR = 0,

	// The requesting app is unavailable.
	AssetPack_APP_UNAVAILABLE,

	// The requested Asset Pack isn't available for this app version.
	AssetPack_UNAVAILABLE,

	// The request is invalid.
	AssetPack_INVALID_REQUEST,

	// The requested download isn't found.
	AssetPack_DOWNLOAD_NOT_FOUND,

	// The Asset Pack API is unavailable.
	AssetPack_API_NOT_AVAILABLE,

	// Network error. Unable to obtain Asset Pack details.
	AssetPack_NETWORK_ERROR,

	// Download not permitted under current device circumstances, e.g. app in
	// background or device not signed into a Google account.
	AssetPack_ACCESS_DENIED,

	// Asset Packs download failed due to insufficient storage.
	AssetPack_INSUFFICIENT_STORAGE,

	// The Play Store app is either not installed or not the official version.
	AssetPack_PLAY_STORE_NOT_FOUND,

	// Returned if showCellularDataConfirmation is called but no Asset Packs are waiting for Wi-Fi.
	AssetPack_NETWORK_UNRESTRICTED,

	// Unknown error downloading Asset Pack.
	AssetPack_INTERNAL_ERROR,

	// The requested operation failed: need to call AssetPackManager_init() first.
	AssetPack_INITIALIZATION_NEEDED,

	// There was an error initializing the Asset Pack API.
	AssetPack_INITIALIZATION_FAILED,

	// The app isn't owned by any user on this device. An app is "owned" if it has been acquired from the Play Store.
	AssetPack_APP_NOT_OWNED,

	// Returned if showConfirmationDialog is called but no asset packs are waiting for user confirmation.
	AssetPack_CONFIRMATION_NOT_REQUIRED,

	// Returned if the app was not installed by Play.
	AssetPack_UNRECOGNIZED_INSTALLATION,
};

// The status associated with Asset Pack download operations.
UENUM(BlueprintType)
enum class EGooglePADDownloadStatus : uint8
{
	// Nothing is known about the Asset Pack.
	AssetPack_UNKNOWN = 0,

	// An AssetPackManager_requestDownload() async request is pending.
	AssetPack_DOWNLOAD_PENDING,

	// The Asset Pack download is in progress.
	AssetPack_DOWNLOADING,

	// The Asset Pack is being transferred to the app.
	AssetPack_TRANSFERRING,

	// Download and transfer are complete; the assets are available to the app.
	AssetPack_DOWNLOAD_COMPLETED,

	// An AssetPackManager_requestDownload() has failed.
	AssetPack_DOWNLOAD_FAILED,

	// Asset Pack download has been canceled.
	AssetPack_DOWNLOAD_CANCELED,

	// The Asset Pack download is waiting for Wi-Fi to proceed.
	AssetPack_WAITING_FOR_WIFI,

	// The Asset Pack isn't installed.
	AssetPack_NOT_INSTALLED,

	// An AssetPackManager_requestInfo() async request started, but the result isn't known yet.
	AssetPack_INFO_PENDING,

	// An AssetPackManager_requestInfo() async request has failed.
	AssetPack_INFO_FAILED,

	// An AssetPackManager_requestRemoval() async request started.
	AssetPack_REMOVAL_PENDING,

	// An AssetPackManager_requestRemoval() async request has failed.
	AssetPack_REMOVAL_FAILED,

	// The Asset Pack download is waiting for user confirmation to proceed.
	AssetPack_REQUIRES_USER_CONFIRMATION,
};

// The method used to store an Asset Pack on the device.
UENUM(BlueprintType)
enum class EGooglePADStorageMethod : uint8
{
	// The Asset Pack is unpacked into a folder containing individual asset files. Assets can be accessed via standard File APIs.
	AssetPack_STORAGE_FILES = 0,

	// The Asset Pack is installed as an APK containing packed asset files. Assets can be accessed via AAssetManager.
	AssetPack_STORAGE_APK,

	// Nothing is known, perhaps due to an error.
	AssetPack_STORAGE_UNKNOWN,

	// The Asset Pack is not installed.
	AssetPack_STORAGE_NOT_INSTALLED,
};

// The status associated with a request to display a cellular data confirmation dialog.
UENUM(BlueprintType)
enum class EGooglePADCellularDataConfirmStatus : uint8
{
	// AssetPackManager_showCellularDataConfirmation() has not been called.
	AssetPack_CONFIRM_UNKNOWN = 0,

	// AssetPackManager_showCellularDataConfirmation() has been called, but the user hasn't made a choice.
	AssetPack_CONFIRM_PENDING,

	// The user approved of downloading Asset Packs over cellular data.
	AssetPack_CONFIRM_USER_APPROVED,

	// The user declined to download Asset Packs over cellular data.
	AssetPack_CONFIRM_USER_CANCELED,
};

// The status associated with a request to display a confirmation dialog.
UENUM(BlueprintType)
enum class EGooglePADConfirmationDialogStatus : uint8
{
	// AssetPackManager_showConfirmationDialog() has not been called.
	AssetPack_CONFIRMATION_DIALOG_UNKNOWN = 0,

	// AssetPackManager_showConfirmationDialog() has been called, but the user hasn't made a choice.
	AssetPack_CONFIRMATION_DIALOG_PENDING,

	// The user approved of downloading asset packs.
	AssetPack_CONFIRMATION_DIALOG_APPROVED,

	// The user declined to download asset packs.
	AssetPack_CONFIRMATION_DIALOG_CANCELED,
};

UCLASS(MinimalAPI)
class UGooglePADFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/** Request information about a set of asset packs */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Request Info"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode RequestInfo(const TArray<FString> AssetPacks);

	/** Request download of a set of asset packs */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Request Download"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode RequestDownload(const TArray<FString> AssetPacks);

	/** Cancel download of a set of asset packs */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Cancel Download"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode CancelDownload(const TArray<FString> AssetPacks);

	/** Get download state handle of an asset pack (release when done) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Download State"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode GetDownloadState(const FString& Name, int32& State);

	/** Release download state resources */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Release Download State"), Category = "GooglePAD")
	static GOOGLEPAD_API void ReleaseDownloadState(const int32 State);

	/** Get download status from a download state */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Download Status"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADDownloadStatus GetDownloadStatus(const int32 State);

	/** Get the number of bytes downloaded from a download state */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Bytes Downloaded"), Category = "GooglePAD")
	static GOOGLEPAD_API int32 GetBytesDownloaded(const int32 State);

	/** Get the total number of bytes to download from a download state */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Total Bytes To Download"), Category = "GooglePAD")
	static GOOGLEPAD_API int32 GetTotalBytesToDownload(const int32 State);

	/** Request removal of an asset pack */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Request Removal"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode RequestRemoval(const FString& Name);

	/** Show confirmation dialog requesting data download over cellular network (DEPRECIATED) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Show Cellular Data Confirmation"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode ShowCellularDataConfirmation();

	/** Get status of cellular confirmation dialog (DEPRECIATED) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Cellular Data Confirmation Status"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode GetShowCellularDataConfirmationStatus(EGooglePADCellularDataConfirmStatus& Status);
		
	/** Show confirmation dialog to start all asset pack downloads in either REQUIRES_USER_CONFIRMATION or WAITING_FOR_WIFI state. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Show Confirmation dialog"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode ShowConfirmationDialog();

	/** Gets the status of confirmation dialog requests */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Cellular Data Confirmation Status"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode GetShowConfirmationDialogStatus(EGooglePADConfirmationDialogStatus& Status);
		
	/** Get location handle of requested asset pack (release when done) */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get AssetPack Location"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADErrorCode GetAssetPackLocation(const FString& Name, int32& Location);

	/** Release location resources */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Release AssetPack Location"), Category = "GooglePAD")
	static GOOGLEPAD_API void ReleaseAssetPackLocation(const int32 Location);

	/** Get storage method from location  */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Storage Method"), Category = "GooglePAD")
	static GOOGLEPAD_API EGooglePADStorageMethod GetStorageMethod(const int32 Location);

	/** Get asset path from from location  */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get Assets Path"), Category = "GooglePAD")
	static GOOGLEPAD_API FString GetAssetsPath(const int32 Location);

public:
	/** initialize java objects and cache them for further usage. called when the module is loaded */
	static GOOGLEPAD_API void Initialize();

	/** releases resources. called when the module is shut down */
	static GOOGLEPAD_API void Shutdown();

private:
#if SUPPORTED_PLATFORM
	/** Conversion functions between enum types */
	static EGooglePADErrorCode ConvertErrorCode(AssetPackErrorCode Code);
	static EGooglePADDownloadStatus ConvertDownloadStatus(AssetPackDownloadStatus Status);
	static EGooglePADCellularDataConfirmStatus ConvertCellarDataConfirmStatus(ShowCellularDataConfirmationStatus Status);
	static EGooglePADConfirmationDialogStatus ConvertConfirmationDialogStatus(ShowConfirmationDialogStatus Status);
	static EGooglePADStorageMethod ConvertStorageMethod(AssetPackStorageMethod Code);
#endif

	/** Helper functions for dealing with asset name arrays */
	static char **ConvertAssetPackNames(TArray<FString> AssetPacks);
	static void ReleaseAssetPackNames(const char **AssetPackNames, int32 NumAssetPacks);

private:
	/** Callback for when the application resumed in the foreground. */
	static void HandleApplicationHasEnteredForeground();

	/** Callback for when the application is being paused in the background. */
	static void HandleApplicationWillEnterBackground();

private:
	/** Foreground/background delegate for pause. */
	static FDelegateHandle PauseHandle;

	/** Foreground/background delegate for resume. */
	static FDelegateHandle ResumeHandle;

	/** Handles for DownloadState */
	static TMap<int32, void*> DownloadStateMap;
	static int32 DownloadStateMapIndex;

	/** Handles for AssetPackLocation */
	static TMap<int32, void*> LocationMap;
	static int32 LocationMapIndex;
};
