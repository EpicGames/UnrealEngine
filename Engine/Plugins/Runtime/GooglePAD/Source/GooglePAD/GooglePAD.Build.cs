// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

public class GooglePAD : ModuleRules
{
	public GooglePAD(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "GooglePAD_APL.xml"));

			string PlayCoreSDKPath = Path.Combine(ModuleDirectory, "..", "ThirdParty", "play-core-native-sdk");
			string PlayCoreIncludePath = Path.Combine(PlayCoreSDKPath, "include");
			string PlayCoreLibPath = Path.Combine(PlayCoreSDKPath, "libs");

			string NDKVersion = "ndk23.0.7599858";
			PublicAdditionalLibraries.Add(Path.Combine(PlayCoreLibPath, "arm64-v8a", NDKVersion, "c++_static", "libplaycore_static.a"));
			PublicAdditionalLibraries.Add(Path.Combine(PlayCoreLibPath, "x86_64", NDKVersion, "c++_static", "libplaycore_static.a"));

			PrivateIncludePaths.Add(PlayCoreIncludePath);
		}
	}
}
