<?xml version="1.0" encoding="utf-8"?>
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<init>
		<log text="GooglePAD Plugin Init"/>
		<setBoolFromProperty result="bEnabled" ini="Engine" section="/Script/GooglePADEditor.GooglePADRuntimeSettings" property="bEnablePlugin" default="false"/>
		<setBoolFromProperty result="bOnlyDistribution" ini="Engine" section="/Script/GooglePADEditor.GooglePADRuntimeSettings" property="bOnlyDistribution" default="true"/>
		<setBoolFromProperty result="bOnlyShipping" ini="Engine" section="/Script/GooglePADEditor.GooglePADRuntimeSettings" property="bOnlyShipping" default="false"/>
		<setBoolFromProperty result="bPackageDataInsideApk" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bPackageDataInsideApk" default="false"/>
		<setBoolFromProperty result="bEnableBundle" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bEnableBundle" default="false"/>
		<setBoolFromProperty result="bAllowPatchOBBFile" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bAllowPatchOBBFile" default="false"/>

		<if condition="bEnabled">
			<true>
				<!-- disable if app bundle disabled -->
				<if condition="bEnableBundle">
					<false>
						<log text="Disabled because not generating AAB bundle"/>
						<setBool result="bEnabled" value="false"/>
					</false>
				</if>

				<!-- disable if only shipping checked and not a shipping build -->
				<setBoolIsEqual result="bIsShipping" arg1="$S(Configuration)" arg2="Shipping"/>
				<if condition="bOnlyShipping">
					<true>
						<if condition="bIsShipping">
							<false>
								<log text="Disabled because not shipping"/>
								<setBool result="bEnabled" value="false"/>
							</false>
						</if>
					</true>
				</if>

				<!-- disable if only distribution checked and not distribution build -->
				<if condition="bOnlyDistribution">
					<true>
						<if condition="Distribution">
							<false>
								<log text="Disabled because not for distribution"/>
								<setBool result="bEnabled" value="false"/>
							</false>
						</if>
					</true>
				</if>

				<!-- make sure OBB is not packaged inside APK -->
				<if condition="bEnabled">
					<true>
						<if condition="bPackageDataInsideApk">
							<true>
								<log text="Disabled because Package Data Inside Apk enabled; disable to use GooglePAD for delivery"/>
								<setBool result="bEnabled" value="false"/>
							</true>
						</if>
						<if condition="bAllowPatchOBBFile">
							<true>
								<log text="Disabled because Allow patch OBB file enabled; disable to use GooglePAD for delivery"/>
								<setBool result="bEnabled" value="false"/>
							</true>
						</if>
					</true>
				</if>
			</true>
		</if>

		<log text="GooglePAD enabled: $B(bEnabled)"/>
	</init>

	<resourceCopies>
		<isDistribution>
			<log text="Copying proguard files for Play Core SDK"/>
			<copyDir src="$S(AbsPluginDir)/../ThirdParty/play-core-native-sdk/proguard"
					dst="$S(BuildDir)/gradle/app/proguard" />
		</isDistribution>
	</resourceCopies>

	<gradleCopies>
	<!--
		<if condition="bEnabled">
			<true>
				<log text="Modifying Gradle version used by replacing gradle-wrapper.properties"/>
				<copyFile src="$S(AbsPluginDir)/../ThirdParty/gradle-wrapper.properties"
							dst="$S(BuildDir)/gradle/gradle/wrapper/gradle-wrapper.properties" />
			</true>
		</if>
	-->
	</gradleCopies>

	<gradleProperties>
	<!--
		<if condition="bEnabled">
			<true>
				<insertValue value="ANDROID_TOOLS_BUILD_GRADLE_VERSION=com.android.tools.build:gradle:4.0.0"/>
				<insertNewline/>
			</true>
		</if>
	-->
	</gradleProperties>

	<settingsGradleAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
<![CDATA[
// generate mainobb assetpack
if (OBB_FILECOUNT.toInteger() > 0) {
	File obbfile = new File(OBB_FILE0)
	if (obbfile.exists()) {
		println 'Creating install-time assetpack for GooglePAD: assetpacks/install-time/obbassets'
		file("assetpacks/install-time/obbassets/src/main/assets").mkdirs()
		def assetBuildGradle = """apply plugin: 'com.android.asset-pack'

assetPack {
	packName = "obbassets"
	dynamicDelivery {
		deliveryType = "install-time"
		instantDeliveryType = "install-time"
	}
}"""
		def assetBuildGradleFile = new File("assetpacks/install-time/obbassets/build.gradle")
		assetBuildGradleFile.write(assetBuildGradle)

		def destobbfile = new File("assetpacks/install-time/obbassets/src/main/assets/main.obb.png")
		if (destobbfile.exists()) {
			destobbfile.delete()
		}
		def srcobbStream = obbfile.newDataInputStream()
		def dstobbStream = destobbfile.newDataOutputStream()
		dstobbStream << srcobbStream
		srcobbStream.close()
		dstobbStream.close()
	}
}

// add the assetpacks
def assetpacksDir = new File("assetpacks/install-time")
if (assetpacksDir.exists()) assetpacksDir.eachDir {
	println ':assetpacks:install-time:' + it.name
	include ':assetpacks:install-time:' + it.name
}
assetpacksDir = new File("assetpacks/fast-follow")
if (assetpacksDir.exists()) assetpacksDir.eachDir {
	println ':assetpacks:fast-follow:' + it.name
	include ':assetpacks:fast-follow:' + it.name
}
assetpacksDir = new File("assetpacks/on-demand")
if (assetpacksDir.exists()) assetpacksDir.eachDir {
	println ':assetpacks:on-demand:' + it.name
	include ':assetpacks:on-demand:' + it.name
}
]]>
</insert>
			</true>
		</if>
	</settingsGradleAdditions>

	<buildGradleAdditions>
		<isDistribution>
			<insert>
android {
	buildTypes {
		release {
			proguardFile "proguard/app_update.pgcfg"
			proguardFile "proguard/asset_delivery.pgcfg"
			proguardFile "proguard/integrity.pgcfg"
			proguardFile "proguard/review.pgcfg"
		}
	}
}

</insert>
		</isDistribution>
		<if condition="bEnabled">
			<true>
				<insert>
<![CDATA[
// for GooglePAD
dependencies {
	implementation('com.google.android.play:asset-delivery:2.3.0')
}

// add the assetpacks
def assetPackSet = []
def assetpacksDir = new File("assetpacks/install-time")
if (assetpacksDir.exists()) assetpacksDir.eachDir {
	assetPackSet.add(':assetpacks:install-time:' + it.name)
}
assetpacksDir = new File("assetpacks/fast-follow")
if (assetpacksDir.exists()) assetpacksDir.eachDir {
	assetPackSet.add(':assetpacks:fast-follow:' + it.name)
}
assetpacksDir = new File("assetpacks/on-demand")
if (assetpacksDir.exists()) assetpacksDir.eachDir {
	assetPackSet.add(':assetpacks:on-demand:' + it.name)
}
println 'AssetPacks: ' + assetPackSet

android {
   assetPacks = assetPackSet
}

]]></insert>			
			</true>
		</if>
	</buildGradleAdditions>

	<gameActivityImportAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
// for GooglePAD
import com.google.android.play.core.assetpacks.AssetLocation;
import com.google.android.play.core.assetpacks.AssetPackLocation;
import com.google.android.play.core.assetpacks.AssetPackManager;
import com.google.android.play.core.assetpacks.AssetPackManagerFactory;
import com.google.android.play.core.assetpacks.AssetPackState;
import com.google.android.play.core.assetpacks.AssetPackStateUpdateListener;
import com.google.android.play.core.assetpacks.AssetPackStates;
import com.google.android.play.core.assetpacks.model.AssetPackStatus;
import com.google.android.gms.tasks.OnCompleteListener;
import com.google.android.gms.tasks.OnSuccessListener;
import com.google.android.gms.tasks.RuntimeExecutionException;
import com.google.android.gms.tasks.Task;

</insert>
			</true>
		</if>
	</gameActivityImportAdditions>

	<gameActivityClassAdditions>
		<if condition="bEnabled">
			<true>
				<insert>
	// for GooglePAD
	private AssetPackManager assetPackManager;
	private AssetPackStateUpdateListener assetPackStateUpdateListener;
	
	public boolean AndroidThunkJava_GooglePAD_Available()
	{
		return true;
	}
</insert>
			</true>
			<false>
				<insert>
	// for GooglePAD
	public boolean AndroidThunkJava_GooglePAD_Available()
	{
		return false;
	}
</insert>
			</false>
		</if>
	</gameActivityClassAdditions>

	<gameActivityOverrideAPKOBBPackaging>
		<if condition="bEnabled">
			<true>
				<insert>
<![CDATA[
		// for GooglePAD (use upfront for main.obb.png)
		assetPackManager = AssetPackManagerFactory.getInstance(this);
		AssetPackLocation obbPackLocation = assetPackManager.getPackLocation("obbassets");
		if (obbPackLocation == null) {
			Log.debug("Unable to locate obbassets asset pack; falling back to best guess location");
			PackageDataInsideApkValue = 1;
			APKPath = APKPath.replaceFirst("base\\.", "split_obbassets\\.");
		} else {
			Log.debug("Found obbassets asset pack");
			AssetLocation obbLocation = assetPackManager.getAssetLocation("obbassets", "main.obb.png");
			if (obbLocation != null) {
				String mainpngPath = obbLocation.path();
				Log.debug("Found main.obb.png asset: " + mainpngPath);
				if (mainpngPath.endsWith(".apk"))
				{
					PackageDataInsideApkValue = 1;
					APKPath = obbLocation.path();
				} else {
					HasAllFiles = true;
					nativeSetObbFilePaths(mainpngPath, "", "", "");
				}
			} else {
				Log.debug("Unable to locate main.obb.png; falling back to best guess location");
				APKPath = APKPath.replaceFirst("base\\.", "split_obbassets\\.");
				PackageDataInsideApkValue = 1;
			}
		}
		// for now release assetPackManager since we reinit with native library
		assetPackManager = null;
]]></insert>
			</true>
		</if>
	</gameActivityOverrideAPKOBBPackaging>

</root>
