// Copyright Epic Games, Inc. All Rights Reserved.

#include "GooglePAD.h"
#include "GooglePADFunctionLibrary.h"
#include "Modules/ModuleManager.h"
#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#endif

#define LOCTEXT_NAMESPACE "FGooglePADModule"

void FGooglePADModule::StartupModule()
{
	// prepare java class and method cache
	UGooglePADFunctionLibrary::Initialize();
}

void FGooglePADModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	UGooglePADFunctionLibrary::Shutdown();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGooglePADModule, GooglePAD)
