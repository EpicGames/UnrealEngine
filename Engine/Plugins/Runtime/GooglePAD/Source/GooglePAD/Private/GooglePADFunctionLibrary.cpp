// Copyright Epic Games, Inc. All Rights Reserved.

#include "GooglePADFunctionLibrary.h"
#include "GooglePAD.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GooglePADFunctionLibrary)

#if SUPPORTED_PLATFORM
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidJNI.h"

extern JavaVM* GJavaVM;
#endif

DEFINE_LOG_CATEGORY(LogGooglePAD);

FDelegateHandle UGooglePADFunctionLibrary::PauseHandle;
FDelegateHandle UGooglePADFunctionLibrary::ResumeHandle;

TMap<int32, void*> UGooglePADFunctionLibrary::DownloadStateMap;
TMap<int32, void*> UGooglePADFunctionLibrary::LocationMap;
int32 UGooglePADFunctionLibrary::DownloadStateMapIndex = 0;
int32 UGooglePADFunctionLibrary::LocationMapIndex = 0;

void UGooglePADFunctionLibrary::Initialize()
{
#if SUPPORTED_PLATFORM
	bool UseGooglePAD = false;
	GConfig->GetBool(TEXT("/Script/GooglePADEditor.GooglePADRuntimeSettings"), TEXT("bEnablePlugin"), UseGooglePAD, GEngineIni);

	if (UseGooglePAD)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			static jmethodID IsGooglePADAvailableFunc = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_GooglePAD_Available", "()Z", false);
			if (IsGooglePADAvailableFunc != nullptr)
			{
				UseGooglePAD = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, IsGooglePADAvailableFunc);
			}
			else
			{
				UseGooglePAD = false;
			}
		}
		else
		{
			UseGooglePAD = false;
		}
	}

	if (UseGooglePAD)
	{
		AssetPackErrorCode result = AssetPackManager_init(GJavaVM, FJavaWrapper::GameActivityThis);
		if (result == ASSET_PACK_NO_ERROR)
		{
			ResumeHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic(&UGooglePADFunctionLibrary::HandleApplicationHasEnteredForeground);
			PauseHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddStatic(&UGooglePADFunctionLibrary::HandleApplicationWillEnterBackground);
			UE_LOGF(LogGooglePAD, Display, "AssetPackManager initialized.");
		}
		else
		{
			UE_LOGF(LogGooglePAD, Error, "Unable to initialize AssetPackManager!");
		}
	}
	else
	{
		UE_LOGF(LogGooglePAD, Display, "GooglePAD disabled.");
	}
#endif
}

void UGooglePADFunctionLibrary::Shutdown()
{
#if SUPPORTED_PLATFORM
	if (ResumeHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
		ResumeHandle.Reset();
	}
	if (PauseHandle.IsValid())
	{
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
		PauseHandle.Reset();
	}
	AssetPackManager_destroy();
#endif
}

void UGooglePADFunctionLibrary::HandleApplicationHasEnteredForeground()
{
#if SUPPORTED_PLATFORM
	AssetPackErrorCode result = AssetPackManager_onResume();
#endif
}

void UGooglePADFunctionLibrary::HandleApplicationWillEnterBackground()
{
#if SUPPORTED_PLATFORM
	AssetPackErrorCode result = AssetPackManager_onPause();
#endif
}

#if SUPPORTED_PLATFORM
EGooglePADErrorCode UGooglePADFunctionLibrary::ConvertErrorCode(AssetPackErrorCode Code)
{
	switch (Code)
	{
		case ASSET_PACK_NO_ERROR:
			return EGooglePADErrorCode::AssetPack_NO_ERROR;
		case ASSET_PACK_APP_UNAVAILABLE:
			return EGooglePADErrorCode::AssetPack_APP_UNAVAILABLE;
		case ASSET_PACK_UNAVAILABLE:
			return EGooglePADErrorCode::AssetPack_UNAVAILABLE;
		case ASSET_PACK_INVALID_REQUEST:
			return EGooglePADErrorCode::AssetPack_INVALID_REQUEST;
		case ASSET_PACK_DOWNLOAD_NOT_FOUND:
			return EGooglePADErrorCode::AssetPack_DOWNLOAD_NOT_FOUND;
		case ASSET_PACK_API_NOT_AVAILABLE:
			return EGooglePADErrorCode::AssetPack_API_NOT_AVAILABLE;
		case ASSET_PACK_NETWORK_ERROR:
			return EGooglePADErrorCode::AssetPack_NETWORK_ERROR;
		case ASSET_PACK_ACCESS_DENIED:
			return EGooglePADErrorCode::AssetPack_ACCESS_DENIED;
		case ASSET_PACK_INSUFFICIENT_STORAGE:
			return EGooglePADErrorCode::AssetPack_INSUFFICIENT_STORAGE;
		case ASSET_PACK_PLAY_STORE_NOT_FOUND:
			return EGooglePADErrorCode::AssetPack_PLAY_STORE_NOT_FOUND;
		case ASSET_PACK_NETWORK_UNRESTRICTED:
			return EGooglePADErrorCode::AssetPack_NETWORK_UNRESTRICTED;
		case ASSET_PACK_APP_NOT_OWNED:
			return EGooglePADErrorCode::AssetPack_APP_NOT_OWNED;
		case ASSET_PACK_CONFIRMATION_NOT_REQUIRED:
			return EGooglePADErrorCode::AssetPack_CONFIRMATION_NOT_REQUIRED;
		case ASSET_PACK_UNRECOGNIZED_INSTALLATION:
			return EGooglePADErrorCode::AssetPack_UNRECOGNIZED_INSTALLATION;
		case ASSET_PACK_INTERNAL_ERROR:
			return EGooglePADErrorCode::AssetPack_INTERNAL_ERROR;
		case ASSET_PACK_INITIALIZATION_NEEDED:
			return EGooglePADErrorCode::AssetPack_INITIALIZATION_NEEDED;
		case ASSET_PACK_INITIALIZATION_FAILED:
			return EGooglePADErrorCode::AssetPack_INITIALIZATION_FAILED;
	}
	return EGooglePADErrorCode::AssetPack_APP_UNAVAILABLE;
}

EGooglePADDownloadStatus UGooglePADFunctionLibrary::ConvertDownloadStatus(AssetPackDownloadStatus Status)
{
	switch (Status)
	{
		case ASSET_PACK_UNKNOWN:
			return EGooglePADDownloadStatus::AssetPack_UNKNOWN;
		case ASSET_PACK_DOWNLOAD_PENDING:
			return EGooglePADDownloadStatus::AssetPack_DOWNLOAD_PENDING;
		case ASSET_PACK_DOWNLOADING:
			return EGooglePADDownloadStatus::AssetPack_DOWNLOADING;
		case ASSET_PACK_TRANSFERRING:
			return EGooglePADDownloadStatus::AssetPack_TRANSFERRING;
		case ASSET_PACK_DOWNLOAD_COMPLETED:
			return EGooglePADDownloadStatus::AssetPack_DOWNLOAD_COMPLETED;
		case ASSET_PACK_DOWNLOAD_FAILED:
			return EGooglePADDownloadStatus::AssetPack_DOWNLOAD_FAILED;
		case ASSET_PACK_DOWNLOAD_CANCELED:
			return EGooglePADDownloadStatus::AssetPack_DOWNLOAD_CANCELED;
		case ASSET_PACK_WAITING_FOR_WIFI:
			return EGooglePADDownloadStatus::AssetPack_WAITING_FOR_WIFI;
		case ASSET_PACK_NOT_INSTALLED:
			return EGooglePADDownloadStatus::AssetPack_NOT_INSTALLED;
		case ASSET_PACK_REQUIRES_USER_CONFIRMATION:
			return EGooglePADDownloadStatus::AssetPack_REQUIRES_USER_CONFIRMATION;
		case ASSET_PACK_INFO_PENDING:
			return EGooglePADDownloadStatus::AssetPack_INFO_PENDING;
		case ASSET_PACK_INFO_FAILED:
			return EGooglePADDownloadStatus::AssetPack_INFO_FAILED;
		case ASSET_PACK_REMOVAL_PENDING:
			return EGooglePADDownloadStatus::AssetPack_REMOVAL_PENDING;
		case ASSET_PACK_REMOVAL_FAILED:
			return EGooglePADDownloadStatus::AssetPack_REMOVAL_FAILED;
	}
	return EGooglePADDownloadStatus::AssetPack_UNKNOWN;
}

EGooglePADCellularDataConfirmStatus UGooglePADFunctionLibrary::ConvertCellarDataConfirmStatus(ShowCellularDataConfirmationStatus Status)
{
	switch (Status)
	{
		case ASSET_PACK_CONFIRM_UNKNOWN:
			return EGooglePADCellularDataConfirmStatus::AssetPack_CONFIRM_UNKNOWN;
		case ASSET_PACK_CONFIRM_PENDING:
			return EGooglePADCellularDataConfirmStatus::AssetPack_CONFIRM_PENDING;
		case ASSET_PACK_CONFIRM_USER_APPROVED:
			return EGooglePADCellularDataConfirmStatus::AssetPack_CONFIRM_USER_APPROVED;
		case ASSET_PACK_CONFIRM_USER_CANCELED:
			return EGooglePADCellularDataConfirmStatus::AssetPack_CONFIRM_USER_CANCELED;
	}
	return EGooglePADCellularDataConfirmStatus::AssetPack_CONFIRM_UNKNOWN;
}

EGooglePADConfirmationDialogStatus UGooglePADFunctionLibrary::ConvertConfirmationDialogStatus(ShowConfirmationDialogStatus Status)
{
	switch (Status)
	{
		case ASSET_PACK_CONFIRMATION_DIALOG_UNKNOWN:
			return EGooglePADConfirmationDialogStatus::AssetPack_CONFIRMATION_DIALOG_UNKNOWN;
		case ASSET_PACK_CONFIRMATION_DIALOG_PENDING:
			return EGooglePADConfirmationDialogStatus::AssetPack_CONFIRMATION_DIALOG_PENDING;
		case ASSET_PACK_CONFIRMATION_DIALOG_APPROVED:
			return EGooglePADConfirmationDialogStatus::AssetPack_CONFIRMATION_DIALOG_APPROVED;
		case ASSET_PACK_CONFIRMATION_DIALOG_CANCELED:
			return EGooglePADConfirmationDialogStatus::AssetPack_CONFIRMATION_DIALOG_CANCELED;
	}
	return EGooglePADConfirmationDialogStatus::AssetPack_CONFIRMATION_DIALOG_UNKNOWN;
}

EGooglePADStorageMethod UGooglePADFunctionLibrary::ConvertStorageMethod(AssetPackStorageMethod Code)
{
	switch (Code)
	{
		case ASSET_PACK_STORAGE_FILES:
			return EGooglePADStorageMethod::AssetPack_STORAGE_FILES;
		case ASSET_PACK_STORAGE_APK:
			return EGooglePADStorageMethod::AssetPack_STORAGE_APK;
		case ASSET_PACK_STORAGE_UNKNOWN:
			return EGooglePADStorageMethod::AssetPack_STORAGE_UNKNOWN;
		case ASSET_PACK_STORAGE_NOT_INSTALLED:
			return EGooglePADStorageMethod::AssetPack_STORAGE_NOT_INSTALLED;
	}
	return EGooglePADStorageMethod::AssetPack_STORAGE_UNKNOWN;
}

#endif

// work around the limited lifespan of TCHAR_TO_UTF8
static char *AllocateAndCopy(char *InString)
{
	if (InString == nullptr)
	{
		return nullptr;
	}

	int InLength = strlen(InString) + 1;
	char *OutString = (char *)FMemory::Malloc(sizeof(char) * InLength);
	FMemory::Memcpy(OutString, InString, InLength);
	return OutString;
}

char **UGooglePADFunctionLibrary::ConvertAssetPackNames(const TArray<FString> AssetPacks)
{
	int32 NumAssetPacks = AssetPacks.Num();
	char **AssetPackNames = (char **)FMemory::Malloc(sizeof(char *) * NumAssetPacks);
	for (int32 Index = 0; Index < NumAssetPacks; Index++)
	{
		AssetPackNames[Index] = AllocateAndCopy(TCHAR_TO_UTF8(*AssetPacks[Index]));
	}
	return AssetPackNames;
}

void UGooglePADFunctionLibrary::ReleaseAssetPackNames(const char **AssetPackNames, int32 NumAssetPacks)
{
	if (AssetPackNames != nullptr)
	{
		for (int32 Index = 0; Index < NumAssetPacks; Index++)
		{
			if (AssetPackNames[Index] != nullptr)
			{
				FMemory::Free((void *)AssetPackNames[Index]);
			}
		}
		FMemory::Free(AssetPackNames);
	}
}

EGooglePADErrorCode UGooglePADFunctionLibrary::RequestInfo(const TArray<FString> AssetPacks)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
#if SUPPORTED_PLATFORM
	const char **AssetPackNames = (const char **)ConvertAssetPackNames(AssetPacks);
	AssetPackErrorCode ErrorCode = AssetPackManager_requestInfo(AssetPackNames, AssetPacks.Num());
	ReleaseAssetPackNames(AssetPackNames, AssetPacks.Num());
	result = ConvertErrorCode(ErrorCode);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::RequestDownload(const TArray<FString> AssetPacks)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
#if SUPPORTED_PLATFORM
	const char **AssetPackNames = (const char **)ConvertAssetPackNames(AssetPacks);
	AssetPackErrorCode ErrorCode = AssetPackManager_requestDownload(AssetPackNames, AssetPacks.Num());
	ReleaseAssetPackNames(AssetPackNames, AssetPacks.Num());
	result = ConvertErrorCode(ErrorCode);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::CancelDownload(const TArray<FString> AssetPacks)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
#if SUPPORTED_PLATFORM
	const char **AssetPackNames = (const char **)ConvertAssetPackNames(AssetPacks);
	AssetPackErrorCode ErrorCode = AssetPackManager_cancelDownload(AssetPackNames, AssetPacks.Num());
	ReleaseAssetPackNames(AssetPackNames, AssetPacks.Num());
	result = ConvertErrorCode(ErrorCode);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::GetDownloadState(const FString& Name, int32& State)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
	State = 0;
#if SUPPORTED_PLATFORM
	AssetPackDownloadState *DownloadState = nullptr;
	AssetPackErrorCode ErrorCode = AssetPackManager_getDownloadState(TCHAR_TO_UTF8(*Name), &DownloadState);
	result = ConvertErrorCode(ErrorCode);
	if (DownloadState != nullptr)
	{
		// find non-zero unused index
		int StartIndex = DownloadStateMapIndex++;
		while (!DownloadStateMapIndex || DownloadStateMap.Find(DownloadStateMapIndex) != nullptr)
		{
			if (++DownloadStateMapIndex == StartIndex)
			{
				UE_LOGF(LogGooglePAD, Error, "Out of handles in UGooglePADFunctionLibrary::GetDownloadState; make sure these are released when done!");
				return EGooglePADErrorCode::AssetPack_UNAVAILABLE;
			}
		}
		DownloadStateMap.Add(DownloadStateMapIndex, DownloadState);
		State = DownloadStateMapIndex;
	}
#endif
	return result;
}

void UGooglePADFunctionLibrary::ReleaseDownloadState(const int32 State)
{
#if SUPPORTED_PLATFORM
	AssetPackDownloadState** DownloadState;
	if (State && (DownloadState = (AssetPackDownloadState**)DownloadStateMap.Find(State)))
	{
		AssetPackDownloadState_destroy(*DownloadState);
		DownloadStateMap.Remove(State);
	}
#endif
}

EGooglePADDownloadStatus UGooglePADFunctionLibrary::GetDownloadStatus(const int32 State)
{
	EGooglePADDownloadStatus result = EGooglePADDownloadStatus::AssetPack_UNKNOWN;
#if SUPPORTED_PLATFORM
	AssetPackDownloadState** DownloadState;
	if (State && (DownloadState = (AssetPackDownloadState**)DownloadStateMap.Find(State)))
	{
		AssetPackDownloadStatus Status = AssetPackDownloadState_getStatus(*DownloadState);
		result = ConvertDownloadStatus(Status);
	}
#endif
	return result;
}

int32 UGooglePADFunctionLibrary::GetBytesDownloaded(const int32 State)
{
#if SUPPORTED_PLATFORM
	AssetPackDownloadState** DownloadState;
	if (State && (DownloadState = (AssetPackDownloadState**)DownloadStateMap.Find(State)))
	{
		return (int32)AssetPackDownloadState_getBytesDownloaded(*DownloadState);
	}
#endif
	return 0;
}

int32 UGooglePADFunctionLibrary::GetTotalBytesToDownload(const int32 State)
{
#if SUPPORTED_PLATFORM
	AssetPackDownloadState** DownloadState;
	if (State && (DownloadState = (AssetPackDownloadState**)DownloadStateMap.Find(State)))
	{
		return (int32)AssetPackDownloadState_getTotalBytesToDownload(*DownloadState);
	}
#endif
	return 0;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::RequestRemoval(const FString& Name)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
#if SUPPORTED_PLATFORM
	AssetPackErrorCode ErrorCode = AssetPackManager_requestRemoval(TCHAR_TO_UTF8(*Name));
	result = ConvertErrorCode(ErrorCode);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::ShowCellularDataConfirmation()
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
#if SUPPORTED_PLATFORM
	UE_LOGF(LogGooglePAD, Warning, "ShowCellularDataConfirmation is deprecated; redirecting to ShowConfirmationDialog!");
	result = ShowConfirmationDialog();
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::GetShowCellularDataConfirmationStatus(EGooglePADCellularDataConfirmStatus& Status)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
	Status = EGooglePADCellularDataConfirmStatus::AssetPack_CONFIRM_UNKNOWN;
#if SUPPORTED_PLATFORM
	EGooglePADConfirmationDialogStatus ConfirmStatus;
	UE_LOGF(LogGooglePAD, Warning, "GetShowCellularDataConfirmation is deprecated; redirecting to GetShowConfirmationDialog!");
	result = GetShowConfirmationDialogStatus(ConfirmStatus);
	Status = static_cast<EGooglePADCellularDataConfirmStatus>(ConfirmStatus);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::ShowConfirmationDialog()
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
#if SUPPORTED_PLATFORM
	AssetPackErrorCode ErrorCode = AssetPackManager_showConfirmationDialog(FJavaWrapper::GameActivityThis);
	result = ConvertErrorCode(ErrorCode);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::GetShowConfirmationDialogStatus(EGooglePADConfirmationDialogStatus& Status)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
	Status = EGooglePADConfirmationDialogStatus::AssetPack_CONFIRMATION_DIALOG_UNKNOWN;
#if SUPPORTED_PLATFORM
	ShowConfirmationDialogStatus ReturnStatus = ASSET_PACK_CONFIRMATION_DIALOG_UNKNOWN;
	AssetPackErrorCode ErrorCode = AssetPackManager_getShowConfirmationDialogStatus(&ReturnStatus);
	result = ConvertErrorCode(ErrorCode);
	Status = ConvertConfirmationDialogStatus(ReturnStatus);
#endif
	return result;
}

EGooglePADErrorCode UGooglePADFunctionLibrary::GetAssetPackLocation(const FString& Name, int32& Location)
{
	EGooglePADErrorCode result = EGooglePADErrorCode::AssetPack_UNAVAILABLE;
	Location = 0;
#if SUPPORTED_PLATFORM
	AssetPackLocation *PackLocation = nullptr;
	AssetPackErrorCode ErrorCode = AssetPackManager_getAssetPackLocation(TCHAR_TO_UTF8(*Name), &PackLocation);
	result = ConvertErrorCode(ErrorCode);
	if (PackLocation != nullptr)
	{
		// find non-zero unused index
		int StartIndex = LocationMapIndex++;
		while (!LocationMapIndex || LocationMap.Find(LocationMapIndex) != nullptr)
		{
			if (++LocationMapIndex == StartIndex)
			{
				UE_LOGF(LogGooglePAD, Error, "Out of handles in UGooglePADFunctionLibrary::GetAssetPackLocation; make sure these are released when done!");
				return EGooglePADErrorCode::AssetPack_UNAVAILABLE;
			}
		}
		LocationMap.Add(LocationMapIndex, PackLocation);
		Location = LocationMapIndex;
	}
#endif
	return result;
}

void UGooglePADFunctionLibrary::ReleaseAssetPackLocation(int32 Location)
{
#if SUPPORTED_PLATFORM
	AssetPackLocation** PackLocation;
	if (Location && (PackLocation = (AssetPackLocation**)LocationMap.Find(Location)))
	{
		AssetPackLocation_destroy(*PackLocation);
		LocationMap.Remove(Location);
	}
#endif
}

EGooglePADStorageMethod UGooglePADFunctionLibrary::GetStorageMethod(const int32 Location)
{
	EGooglePADStorageMethod result = EGooglePADStorageMethod::AssetPack_STORAGE_UNKNOWN;
#if SUPPORTED_PLATFORM
	AssetPackLocation** PackLocation;
	if (Location && (PackLocation = (AssetPackLocation**)LocationMap.Find(Location)))
	{
		AssetPackStorageMethod StorageMethod = AssetPackLocation_getStorageMethod(*PackLocation);
		result = ConvertStorageMethod(StorageMethod);
	}
#endif
	return result;
}

FString UGooglePADFunctionLibrary::GetAssetsPath(const int32 Location)
{
	FString result = TEXT("");
#if SUPPORTED_PLATFORM
	AssetPackLocation** PackLocation;
	if (Location && (PackLocation = (AssetPackLocation**)LocationMap.Find(Location)))
	{
		const char *Path = AssetPackLocation_getAssetsPath(*PackLocation);
		result = FString(UTF8_TO_TCHAR(Path));
	}
#endif
	return result;
}
