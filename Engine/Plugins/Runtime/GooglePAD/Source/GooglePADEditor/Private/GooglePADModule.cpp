// Copyright Epic Games, Inc. All Rights Reserved.

#include "GooglePADEditor.h"
#include "ISettingsModule.h"
#include "GooglePADRuntimeSettings.h"
#include "Modules/ModuleManager.h"

/**
 * Implements the GooglePADEditor module.
 */

#define LOCTEXT_NAMESPACE "GooglePAD"

void FGooglePADEditorModule::StartupModule()
{
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)	
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "GooglePAD",
			LOCTEXT("GooglePADSettingsName", "GooglePAD"),
			LOCTEXT("GooglePADSettingsDescription", "Project settings for GooglePAD plugin"),
			GetMutableDefault<UGooglePADRuntimeSettings>()
		);
	}
}

void FGooglePADEditorModule::ShutdownModule()
{
	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
       		SettingsModule->UnregisterSettings("Project", "Plugins", "GooglePAD");
	}
}


IMPLEMENT_MODULE(FGooglePADEditorModule, GooglePADEditor);

#undef LOCTEXT_NAMESPACE
