// Copyright Epic Games, Inc. All Rights Reserved.

#include "GooglePADRuntimeSettings.h"

//////////////////////////////////////////////////////////////////////////
// UGooglePADRuntimeSettings

#include UE_INLINE_GENERATED_CPP_BY_NAME(GooglePADRuntimeSettings)

UGooglePADRuntimeSettings::UGooglePADRuntimeSettings(const FObjectInitializer& ObjectInitializer)
        : Super(ObjectInitializer)
        , bEnablePlugin(false)
        , bOnlyDistribution(true)
		, bOnlyShipping(false)
{
}
