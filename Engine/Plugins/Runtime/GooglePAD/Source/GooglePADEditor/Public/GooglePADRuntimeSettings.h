// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "GooglePADRuntimeSettings.generated.h"

/**
* Implements the settings for the GooglePAD plugin.
*/
UCLASS(Config = Engine, DefaultConfig)
class UGooglePADRuntimeSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	// Enable GooglePAD plugin
	UPROPERTY(EditAnywhere, config, Category = Packaging)
	bool bEnablePlugin;

	// Only for distribution builds
	UPROPERTY(EditAnywhere, config, Category = Packaging)
	bool bOnlyDistribution;

	// Only for shipping builds
	UPROPERTY(EditAnywhere, config, Category = Packaging)
	bool bOnlyShipping;
};
